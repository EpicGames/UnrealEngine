{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Camera Shake Previewer",
	"Description" : "Adds a new panel, accessible from the Level Editor, which lets the user preview camera shakes in editor viewports.",
	"Category" : "Cameras",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Plugins" :
	[
		{
			"Name": "GameplayCameras",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "CameraShakePreviewer",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		}
	]
}
