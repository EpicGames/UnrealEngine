// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CameraShakePreviewer : ModuleRules
{
	public CameraShakePreviewer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"WorkspaceMenuStructure",
				}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"SlateCore",
				"AppFramework",
				"GameplayCameras",
				"LevelEditor",
				"MovieScene",
				"MovieSceneTracks",
				"PropertyEditor",
				"Slate",
				"ToolMenus",
				"EditorFramework",
				"UnrealEd",
			}
		);

		PublicDependencyModuleNames.AddRange(new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
			}
		);
    }
}

