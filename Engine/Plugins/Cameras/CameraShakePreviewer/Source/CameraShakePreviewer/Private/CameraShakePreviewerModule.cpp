// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraShakePreviewerModule.h"
#include "Editor/UnrealEdTypes.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"
#include "SCameraShakePreviewer.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "SLevelViewport.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LevelEditorMenuContext.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"

#define LOCTEXT_NAMESPACE "CameraShakePreviewer"

IMPLEMENT_MODULE(FCameraShakePreviewerModule, CameraShakePreviewer);

namespace UE::CameraShakePreviewer::Private
{
static const FName LevelEditorModuleName("LevelEditor");
static const FName LevelEditorCameraShakePreviewerTab("CameraShakePreviewer");

TSharedPtr<SLevelViewport> GetLevelViewport()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(LevelEditorModuleName);

	if (TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetLevelEditorInstance().Pin())
	{
		return LevelEditor->GetActiveViewportInterface();
	}

	return nullptr;
}

FLevelEditorViewportClient* GetPerspectiveLevelEditorViewportClient()
{
	if (TSharedPtr<SLevelViewport> ViewportInterface = GetLevelViewport())
	{
		FLevelEditorViewportClient* ViewportClient = &ViewportInterface->GetLevelViewportClient();
		if (ViewportClient->ViewportType == ELevelViewportType::LVT_Perspective)
		{
			return ViewportClient;
		}
	}

	return nullptr;
}

TSharedPtr<FEditorViewportClient> GetLevelEditorViewportClient()
{
	if (TSharedPtr<SLevelViewport> ViewportInterface = GetLevelViewport())
	{
		return ViewportInterface->GetViewportClient();
	}

	return nullptr;
}
} // namespace UE::CameraShakePreviewer::Private

/**
 * Editor commands for the camera shake preview tool.
 */
class FCameraShakePreviewerCommands : public TCommands<FCameraShakePreviewerCommands>
{
public:
	FCameraShakePreviewerCommands()
		: TCommands<FCameraShakePreviewerCommands>(
				TEXT("CameraShakePreviewer"),
				LOCTEXT("CameraShakePreviewerContextDescription", "Camera Shake Previewer"),
				TEXT("EditorViewport"),
				FAppStyle::GetAppStyleSetName())
	{
	}

	virtual void RegisterCommands() override
	{
		UI_COMMAND(ToggleCameraShakesPreview, "Allow Camera Shakes", "If enabled, allows the camera shakes previewer panel to apply shakes to this viewport", EUserInterfaceActionType::ToggleButton, FInputChord());
	}

	TSharedPtr<FUICommandInfo> ToggleCameraShakesPreview;
};


void FCameraShakePreviewerModule::StartupModule()
{
	FCameraShakePreviewerCommands::Register();

	if (ensure(FModuleManager::Get().IsModuleLoaded(UE::CameraShakePreviewer::Private::LevelEditorModuleName)))
	{
		RegisterEditorTab();
		RegisterViewportOptionMenuExtender();
	}
}

void FCameraShakePreviewerModule::ShutdownModule()
{
	UnregisterViewportOptionMenuExtender();
	UnregisterEditorTab();

	FCameraShakePreviewerCommands::Unregister();
}

void FCameraShakePreviewerModule::RegisterEditorTab()
{
	FLevelEditorModule& LevelEditorModule =
		FModuleManager::LoadModuleChecked<FLevelEditorModule>(UE::CameraShakePreviewer::Private::LevelEditorModuleName);

	LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddLambda([]()
	{
		// Add a new entry in the level editor's "Window" menu, which lets the user open the camera shake preview tool.
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(
				UE::CameraShakePreviewer::Private::LevelEditorModuleName
			);
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

		const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
		const FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "LevelViewport.ToggleActorPilotCameraView");

		LevelEditorTabManager->RegisterTabSpawner("CameraShakePreviewer", FOnSpawnTab::CreateStatic(&FCameraShakePreviewerModule::CreateCameraShakePreviewerTab))
			.SetDisplayName(LOCTEXT("CameraShakePreviewer", "Camera Shake Previewer"))
			.SetTooltipText(LOCTEXT("CameraShakePreviewerTooltipText", "Open the camera shake preview panel."))
			.SetIcon(Icon)
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCinematicsCategory());
	});
}

void FCameraShakePreviewerModule::UnregisterEditorTab()
{
	if (LevelEditorTabManagerChangedHandle.IsValid())
	{
		FLevelEditorModule& LevelEditorModule =
			FModuleManager::GetModuleChecked<FLevelEditorModule>(UE::CameraShakePreviewer::Private::LevelEditorModuleName);
		LevelEditorModule.OnTabManagerChanged().Remove(LevelEditorTabManagerChangedHandle);
	}
}

void FCameraShakePreviewerModule::RegisterViewportOptionMenuExtender()
{
	// Register a callback for adding a "Show Camera Shakes" option in the viewport options menu.
	FLevelEditorModule& LevelEditorModule =
		FModuleManager::LoadModuleChecked<FLevelEditorModule>(UE::CameraShakePreviewer::Private::LevelEditorModuleName);

	FLevelEditorModule::FLevelEditorMenuExtender Extender = FLevelEditorModule::FLevelEditorMenuExtender::CreateRaw(this, &FCameraShakePreviewerModule::OnExtendLevelViewportOptionMenu);
	LevelEditorModule.GetAllLevelViewportOptionsMenuExtenders().Add(Extender);
	ViewportOptionsMenuExtenderHandle = LevelEditorModule.GetAllLevelViewportOptionsMenuExtenders().Last().GetHandle();

	GEditor->OnLevelViewportClientListChanged().AddRaw(this, &FCameraShakePreviewerModule::OnLevelViewportClientListChanged);

	OnLevelViewportClientListChanged();
}

void FCameraShakePreviewerModule::UnregisterViewportOptionMenuExtender()
{
	UToolMenus::UnregisterOwner(this);

	FLevelEditorModule& LevelEditorModule =
		FModuleManager::GetModuleChecked<FLevelEditorModule>(UE::CameraShakePreviewer::Private::LevelEditorModuleName);
	LevelEditorModule.GetAllLevelViewportOptionsMenuExtenders().RemoveAll([this](const FLevelEditorModule::FLevelEditorMenuExtender& Extender) { return Extender.GetHandle() == ViewportOptionsMenuExtenderHandle; });

	if (GEditor)
	{
		GEditor->OnLevelViewportClientListChanged().RemoveAll(this);
	}
}

TSharedRef<FExtender> FCameraShakePreviewerModule::OnExtendLevelViewportOptionMenu(const TSharedRef<FUICommandList> CommandList)
{
	TSharedRef<FExtender> Extender(new FExtender());

	// Legacy extension hook
	Extender->AddMenuExtension(
		"LevelViewportViewportOptions2",
		EExtensionHook::After,
		nullptr,
		FMenuExtensionDelegate::CreateRaw(this, &FCameraShakePreviewerModule::CreateCameraShakeToggleOption)
	);

	// Adding separate extension hook for new toolbar
	Extender->AddMenuExtension(
		"CameraOptions",
		EExtensionHook::After,
		nullptr,
		FMenuExtensionDelegate::CreateRaw(this, &FCameraShakePreviewerModule::CreateCameraShakeToggleOption)
	);

	return Extender;
}

void FCameraShakePreviewerModule::OnLevelViewportClientListChanged()
{
	const TArray<FLevelEditorViewportClient*> LevelViewportClients = GEditor->GetLevelViewportClients();
	// Remove viewports that don't exist anymore.
	for (auto It = ViewportInfos.CreateIterator(); It; ++It)
	{
		if (!LevelViewportClients.Contains(It.Key()))
		{
			It.RemoveCurrent();
		}
	}
	// Add recently added viewports that we don't know about yet.
	for (FLevelEditorViewportClient* LevelViewportClient : LevelViewportClients)
	{
		if (!ViewportInfos.Contains(LevelViewportClient))
		{
			ViewportInfos.Add(LevelViewportClient, FViewportInfo{ false });
		}
	}
}

void FCameraShakePreviewerModule::CreateCameraShakeToggleOption(FMenuBuilder& InMenuBuilder)
{
	FUIAction ToggleCameraShakeAction;

	ToggleCameraShakeAction.ExecuteAction.BindLambda(
		[this]()
		{
			if (FLevelEditorViewportClient* ViewportClient =
					UE::CameraShakePreviewer::Private::GetPerspectiveLevelEditorViewportClient())
			{
				ToggleCameraShakesPreview(ViewportClient);
			}
		}
	);

	ToggleCameraShakeAction.GetActionCheckState.BindLambda(
		[this]() -> ECheckBoxState
		{
			if (FLevelEditorViewportClient* ViewportClient =
					UE::CameraShakePreviewer::Private::GetPerspectiveLevelEditorViewportClient())
			{
				if (HasCameraShakesPreview(ViewportClient))
				{
					return ECheckBoxState::Checked;
				}
			}

			return ECheckBoxState::Unchecked;
		}
	);

	// Show this entry only if viewport is perspective
	TAttribute<EVisibility> VisibilityOverride = UE::UnrealEd::GetPerspectiveOnlyVisibility(UE::CameraShakePreviewer::Private::GetLevelEditorViewportClient());

	TSharedPtr<FUICommandInfo> ToggleCameraShakesPreview = FCameraShakePreviewerCommands::Get().ToggleCameraShakesPreview;
	InMenuBuilder.AddMenuEntry(
		ToggleCameraShakesPreview->GetLabel(),
		ToggleCameraShakesPreview->GetDescription(),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelViewport.ToggleCameraShakePreview"),
		ToggleCameraShakeAction,
		NAME_None,
		EUserInterfaceActionType::ToggleButton,
		NAME_None,
		FText(),
		VisibilityOverride
	);
}

void FCameraShakePreviewerModule::ToggleCameraShakesPreview(FLevelEditorViewportClient* ViewportClient)
{
	if (FViewportInfo* ViewportInfo = ViewportInfos.Find(ViewportClient))
	{
		ViewportInfo->bPreviewCameraShakes = !ViewportInfo->bPreviewCameraShakes;
		OnTogglePreviewCameraShakes.Broadcast(FTogglePreviewCameraShakesParams{ ViewportClient, ViewportInfo->bPreviewCameraShakes });
	}
}

bool FCameraShakePreviewerModule::HasCameraShakesPreview(FLevelEditorViewportClient* ViewportClient) const
{
	const FViewportInfo* ViewportInfo = ViewportInfos.Find(ViewportClient);
	return ViewportInfo != nullptr && ViewportInfo->bPreviewCameraShakes;
}

TSharedRef<SDockTab> FCameraShakePreviewerModule::CreateCameraShakePreviewerTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			SNew(SCameraShakePreviewer)
		];
}

#undef LOCTEXT_NAMESPACE
