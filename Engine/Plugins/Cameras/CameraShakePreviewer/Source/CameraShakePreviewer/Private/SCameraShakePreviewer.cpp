// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCameraShakePreviewer.h"
#include "Camera/CameraModifier_CameraShake.h"
#include "Camera/CameraShakeBase.h"
#include "Camera/CameraShakeSourceActor.h"
#include "Camera/CameraShakeSourceComponent.h"
#include "CameraShakePreviewerModule.h"
#include "EditorSupportDelegates.h"
#include "Engine/Level.h"
#include "LevelEditorViewport.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "CameraShakePreviewer"

namespace UE::Sequencer
{

UWorld* FindCameraShakePreviewerWorld()
{
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::Editor)
		{
			return Context.World();
		}
	}
	return nullptr;
}

} // namespace UE::MovieScene

/**
 * Data struct for each entry in the panel's main list.
 */
struct FCameraShakeData
	: public TSharedFromThis<FCameraShakeData>
	, public FGCObject
{
	TSubclassOf<UCameraShakeBase> ShakeClass;
	TObjectPtr<UCameraShakeBase> ShakeInstance = nullptr;
	bool bIsPlaying = false;
	bool bIsHidden = false;

	TWeakObjectPtr<UCameraShakeSourceComponent> SourceComponent;

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(ShakeInstance);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FCameraShakeData");
	}
};

FCameraShakePreviewUpdater::FCameraShakePreviewUpdater(UWorld* InWorld)
	: Previewer(InWorld)
{
}

FCameraShakePreviewUpdater::~FCameraShakePreviewUpdater()
{
	Previewer.UnRegisterViewModifiers();
}

void FCameraShakePreviewUpdater::Tick(float DeltaTime)
{
	Previewer.Update(DeltaTime, true);
}

UCameraShakeBase* FCameraShakePreviewUpdater::AddCameraShake(TSubclassOf<UCameraShakeBase> ShakeClass, const FAddCameraShakeParams& Params)
{
	FCameraShakePreviewerAddParams ActualParams;
	ActualParams.ShakeClass = ShakeClass;
	ActualParams.SourceComponent = Params.SourceComponent;
	ActualParams.Scale = Params.Scale;
	ActualParams.PlaySpace = Params.PlaySpace;
	ActualParams.UserPlaySpaceRot = Params.UserPlaySpaceRot;
	return Previewer.AddCameraShake(ActualParams);
}

void FCameraShakePreviewUpdater::RemoveAllCameraShakesFromSource(const UCameraShakeSourceComponent* SourceComponent)
{
	Previewer.RemoveAllCameraShakesFromSource(SourceComponent);
}

void FCameraShakePreviewUpdater::GetActiveCameraShakes(TArray<FActiveCameraShakeInfo>& ActiveCameraShakes) const
{
	Previewer.GetActiveCameraShakes(ActiveCameraShakes);
}

void FCameraShakePreviewUpdater::RemoveCameraShake(UCameraShakeBase* ShakeInstance)
{
	Previewer.RemoveCameraShake(ShakeInstance);
}

void FCameraShakePreviewUpdater::RemoveAllCameraShakes()
{
	Previewer.RemoveAllCameraShakes();
}

/**
 * The UI for each entry in the panel's main list.
 */
class SCameraShakeRow : public SMultiColumnTableRow<TSharedPtr<FCameraShakeData>>
{
public:
	SLATE_BEGIN_ARGS(SCameraShakeRow) {}
	SLATE_ARGUMENT(TSharedPtr<FCameraShakeData>, CameraShake)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		CameraShake = Args._CameraShake;

		FSuperRowType::Construct(
			FSuperRowType::FArguments().Padding(1.0f),
			OwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == TEXT("CameraShakeName"))
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(12.0f, 10.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SCameraShakeRow::GetCameraShakeName)
				];
		}
		else if (ColumnName == TEXT("SceneActorName"))
		{
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(12.0f, 10.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SCameraShakeRow::GetOwnerActorName)
				];
		}
		else if (ColumnName == TEXT("Status"))
		{
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(12.0f, 10.0f)
				.VAlign(VAlign_Bottom)
				[
					SNew(STextBlock)
					.Text(this, &SCameraShakeRow::GetCameraShakeStatus)
				];
		}
		return SNullWidget::NullWidget;
	}

private:
	FText GetCameraShakeName() const
	{
		if (CameraShake->ShakeClass != nullptr)
		{
			return FText::FromString(CameraShake->ShakeClass->GetName());
		}
		return FText::FromString("<None>");
	}

	FText GetOwnerActorName() const
	{
		if (CameraShake->SourceComponent.IsValid())
		{
			if (const AActor* Actor = CameraShake->SourceComponent->GetOwner())
			{
				return FText::FromString(Actor->GetName());
			}
		}
		return FText::GetEmpty();
	}

	FText GetCameraShakeStatus() const
	{
		if (CameraShake->bIsHidden)
		{
			return FText::FromString("(Hidden)");
		}
		return CameraShake->bIsPlaying ? FText::FromString("Playing") : FText::FromString("Stopped");
	}

	TSharedPtr<FCameraShakeData> CameraShake;
};

void SCameraShakePreviewer::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SAssignNew(CameraShakesListView, SListView<TSharedPtr<FCameraShakeData>>)
			.ListItemsSource(&CameraShakes)
			.OnGenerateRow(this, &SCameraShakePreviewer::OnCameraShakesListGenerateRowWidget)
			.OnSelectionChanged(this, &SCameraShakePreviewer::OnCameraShakesListSectionChanged)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column("CameraShakeName")
				.DefaultLabel(LOCTEXT("CameraShakeName", "Camera Shake Name"))
				.FillWidth(0.3f)
				+SHeaderRow::Column("SceneActorName")
				.DefaultLabel(LOCTEXT("SceneActorName", "Scene Actor Name"))
				.FillWidth(0.3f)
				+SHeaderRow::Column("Status")
				.DefaultLabel(LOCTEXT("Status", "Status"))
				.FillWidth(0.3f)
			)
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.Padding(1.0f)
			[
				SNew(SWrapBox)
				.UseAllottedSize(true)
				+SWrapBox::Slot()
				.Padding(FMargin(2.0f))
				[
					SNew(SButton)
					.OnClicked(this, &SCameraShakePreviewer::OnPlayStopAllShakes)
					.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("PlayStopAllShakes", "Play/Stop All"))
					.ToolTipText(LOCTEXT("PlayStopAllShakesTooltip", "Play/stop all shakes in the list"))
				]
				+SWrapBox::Slot()
				.Padding(FMargin(2.0f))
				[
					SAssignNew(PlayStopSelectedButton, SButton)
					.OnClicked(this, &SCameraShakePreviewer::OnPlayStopSelectedShake)
					.IsEnabled(false)
					.Text(LOCTEXT("PlayStopSelectedShake", "Play/Stop Selected"))
					.ToolTipText(LOCTEXT("PlayStopSelectedShakeTooltip", "Play/stop select shake"))
				]
				+ SWrapBox::Slot()
				.Padding(FMargin(2.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ActiveViewport", "Active Viewport:"))
				]
				+SWrapBox::Slot()
				.Padding(FMargin(2.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SCameraShakePreviewer::GetActiveViewportName)
				]
				+ SWrapBox::Slot()
				.Padding(FMargin(2.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), TEXT("Log.Warning"))
					.Text(this, &SCameraShakePreviewer::GetActiveViewportWarnings)
				]
			]
		]
	];

	// Listen to when the user toggles support for previewing camera shakes. We'll want to add/remove our view modifiers on
	// the affected viewport when it happens.
	CameraShakePreviewerModule = &FModuleManager::GetModuleChecked<FCameraShakePreviewerModule>("CameraShakePreviewer");
	CameraShakePreviewerModule->OnTogglePreviewCameraShakes.AddSP(this, &SCameraShakePreviewer::OnTogglePreviewCameraShakes);

	// Register callbacks for when stuff happens that might affect the list of shake sources.
	FEditorDelegates::MapChange.AddSP(this, &SCameraShakePreviewer::OnMapChange);
	FEditorDelegates::NewCurrentLevel.AddSP(this, &SCameraShakePreviewer::OnNewCurrentLevel);
	
	FWorldDelegates::LevelAddedToWorld.AddSP(this, &SCameraShakePreviewer::OnLevelAdded);
	FWorldDelegates::LevelRemovedFromWorld.AddSP(this, &SCameraShakePreviewer::OnLevelRemoved);

	if (GEngine != nullptr)
	{
		GEngine->OnLevelActorListChanged().AddSP(this, &SCameraShakePreviewer::OnLevelActorListChanged);
		GEngine->OnLevelActorAdded().AddSP(this, &SCameraShakePreviewer::OnLevelActorsAdded);
		GEngine->OnLevelActorDeleted().AddSP(this, &SCameraShakePreviewer::OnLevelActorsRemoved);
	}
	if (GEditor != nullptr)
	{
		GEditor->RegisterForUndo(this);
		GEditor->OnLevelViewportClientListChanged().AddSP(this, &SCameraShakePreviewer::OnLevelViewportClientListChanged);
	}

	// Populate the main list based on the current level.
	UpdateActiveViewportAndWorld();
	Populate();
	bNeedsRefresh = false;
}

SCameraShakePreviewer::~SCameraShakePreviewer()
{
	CameraShakePreviewUpdater = nullptr;

	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::NewCurrentLevel.RemoveAll(this);
	
	if (GEditor != nullptr)
	{
		GEditor->UnregisterForUndo(this);
		GEditor->OnLevelViewportClientListChanged().RemoveAll(this);
	}
	if (GEngine != nullptr)
	{
		GEngine->OnLevelActorListChanged().RemoveAll(this);
		GEngine->OnLevelActorAdded().RemoveAll(this);
		GEngine->OnLevelActorDeleted().RemoveAll(this);
	}
	
	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);

	CameraShakePreviewerModule->OnTogglePreviewCameraShakes.RemoveAll(this);
}

TSharedRef<ITableRow> SCameraShakePreviewer::OnCameraShakesListGenerateRowWidget(TSharedPtr<FCameraShakeData> CameraShake, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SCameraShakeRow, OwnerTable)
		.CameraShake(CameraShake);
}

void SCameraShakePreviewer::Populate()
{
	UWorld* CurrentWorld =  WeakCurrentWorld.Get();
	if (!ensureMsgf(CurrentWorld, TEXT("Could not find current world instance.")))
	{
		return;
	}

	// Get all the shake sources from the level.
	TArray<UCameraShakeSourceComponent*> ShakeSourceComponents;
	for (ULevel* Level : CurrentWorld->GetLevels())
	{
		if (!Level->bIsVisible)
		{
			continue;
		}

		for (AActor* Actor : Level->Actors)
		{
			ACameraShakeSourceActor* ShakeSourceActor = Cast<ACameraShakeSourceActor>(Actor);
			if (ShakeSourceActor != nullptr)
			{
				if (UCameraShakeSourceComponent* ShakeSourceComponent = ShakeSourceActor->GetCameraShakeSourceComponent())
				{
					ShakeSourceComponents.Add(ShakeSourceComponent);
				}
			}
		}
	}
	TSet<UCameraShakeSourceComponent*> ShakeSourceComponentsSet(ShakeSourceComponents);

	// Get all the shake sources we already knew about.
	TSet<TSharedPtr<FCameraShakeData>> RemovedShakes;
	TSet<UCameraShakeSourceComponent*> PreviousShakeSourceComponentsSet;
	for (TSharedPtr<FCameraShakeData> CameraShake : CameraShakes)
	{
		if (CameraShake->SourceComponent.IsValid())
		{
			UCameraShakeSourceComponent* ShakeSourceComponent = CameraShake->SourceComponent.Get();
			if (ShakeSourceComponentsSet.Contains(ShakeSourceComponent))
			{
				PreviousShakeSourceComponentsSet.Add(ShakeSourceComponent);
			}
			else
			{
				RemovedShakes.Add(CameraShake);
			}
		}
		else if (CameraShake->SourceComponent.IsStale())
		{
			RemovedShakes.Add(CameraShake);
		}
	}

	// Remove the shakes that have sources that were destroyed. We don't need to stop them if they were running, 
	// the `CameraModifier_CameraShake` class will clean those up automatically.
	for (TSharedPtr<FCameraShakeData> RemovedShake : RemovedShakes)
	{
		if (RemovedShake->SourceComponent.IsValid())
		{
			CameraShakePreviewUpdater->RemoveAllCameraShakesFromSource(RemovedShake->SourceComponent.Get());
		}
		CameraShakes.Remove(RemovedShake);
	}

	// Add new camera shake sources to the list.
	TSet<UCameraShakeSourceComponent*> NewShakeSourceComponents = ShakeSourceComponentsSet.Difference(PreviousShakeSourceComponentsSet);
	for (UCameraShakeSourceComponent* ShakeSourceComponent : NewShakeSourceComponents)
	{
		TSharedPtr<FCameraShakeData> CameraShake = MakeShared<FCameraShakeData>();
		CameraShake->ShakeClass = ShakeSourceComponent->CameraShake;
		CameraShake->SourceComponent = ShakeSourceComponent;
		CameraShakes.Add(CameraShake);
	}

	CameraShakesListView->RequestListRefresh();
}

void SCameraShakePreviewer::Refresh()
{
	bNeedsRefresh = true;
}

void SCameraShakePreviewer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	// See if the world or active viewport have changed.
	UpdateActiveViewportAndWorld();

	// Update our list of camera shakes if needed.
	if (bNeedsRefresh)
	{
		Populate();
		bNeedsRefresh = false;
	}

	// Update playing information on our camera shakes.
	TArray<FActiveCameraShakeInfo> ActiveCameraShakes;
	CameraShakePreviewUpdater->GetActiveCameraShakes(ActiveCameraShakes);
	for (TSharedPtr<FCameraShakeData> CameraShake : CameraShakes)
	{
		if (CameraShake->SourceComponent.IsValid())
		{
			UCameraShakeSourceComponent* ShakeSourceComponent = CameraShake->SourceComponent.Get();

			// Handle the case where the actor was hidden: stop it if it was playing, flag it as hidden.
			if (AActor* ShakeSourceActor = ShakeSourceComponent->GetOwner())
			{
				const bool bIsHidden = ShakeSourceActor->IsHiddenEd();
				CameraShake->bIsHidden = bIsHidden;
				if (bIsHidden && CameraShake->bIsPlaying)
				{
					CameraShakePreviewUpdater->RemoveAllCameraShakesFromSource(ShakeSourceComponent);
					CameraShake->bIsPlaying = false;
				}
			}

			// Handle the case where the user has changed the shake type on an existing shake source.
			if (ShakeSourceComponent->CameraShake != CameraShake->ShakeClass)
			{
				// Yep, it's a different class now. Let's stop any running instance of the old class,
				// and figure out if we should start playing the new class right away (if the old class instance
				// was running until now).
				CameraShake->ShakeClass = ShakeSourceComponent->CameraShake;
				CameraShake->ShakeInstance = nullptr;

				CameraShakePreviewUpdater->RemoveAllCameraShakesFromSource(ShakeSourceComponent);

				if (CameraShake->bIsPlaying)
				{
					if (ShakeSourceComponent->CameraShake != nullptr)
					{
						PlayCameraShake(CameraShake);
					}
					else
					{
						CameraShake->bIsPlaying = false;
					}
				}
			}

			// Check if the shake is still playing.
			if (CameraShake->bIsPlaying && CameraShake->ShakeInstance != nullptr)
			{
				if (!ActiveCameraShakes.ContainsByPredicate(
							[CameraShake](const FActiveCameraShakeInfo& ShakeInfo)
							{
							return ShakeInfo.ShakeInstance == CameraShake->ShakeInstance;
							}))
				{
					CameraShake->bIsPlaying = false;
				}
			}
		}
	}
}

void SCameraShakePreviewer::UpdateActiveViewportAndWorld()
{
	if (!GEditor)
	{
		ActiveViewportIndex = INDEX_NONE;
		WeakCurrentWorld = nullptr;
		CameraShakePreviewUpdater.Reset();
		return;
	}

	// Update the active viewport and index. This is used for showing viewport info in the UI.
	ActiveViewportIndex = 0;
	ActiveViewportClient = nullptr;
	FViewport* const ActiveViewport = GEditor->GetActiveViewport();
	for (FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
	{
		if (ViewportClient->Viewport == ActiveViewport)
		{
			ActiveViewportClient = ViewportClient;
			break;
		}
		++ActiveViewportIndex;
	}
	if (!ActiveViewportClient)
	{
		ActiveViewportIndex = INDEX_NONE;
	}

	// Always use the first editor world we can find. If this is a different world than what we
	// were looking at before, re-create the shake previewer.
	UWorld* const PreviousWorld = WeakCurrentWorld.Get();
	UWorld* const NewWorld = UE::Sequencer::FindCameraShakePreviewerWorld();
	if (PreviousWorld != NewWorld)
	{
		if (NewWorld)
		{
			CameraShakePreviewUpdater.Reset(new FCameraShakePreviewUpdater(NewWorld));

			// Immediately register the previewer with any viewport that wants shaking.
			CameraShakePreviewUpdater->GetPreviewer().RegisterViewModifiers(
					[this, NewWorld](FLevelEditorViewportClient* LevelVC)
					{
						return LevelVC->GetWorld() == NewWorld && 
							CameraShakePreviewerModule->HasCameraShakesPreview(LevelVC);
					},
					// Pass false to help the Mac compiler along.
					false);
		}
		else
		{
			CameraShakePreviewUpdater.Reset();
		}
	}
	WeakCurrentWorld = NewWorld;
}

void SCameraShakePreviewer::OnTogglePreviewCameraShakes(const FTogglePreviewCameraShakesParams& Params)
{
	if (Params.ViewportClient && Params.ViewportClient->GetWorld() == WeakCurrentWorld)
	{
		if (Params.bPreviewCameraShakes)
		{
			CameraShakePreviewUpdater->GetPreviewer().RegisterViewModifier(Params.ViewportClient);
		}
		else if (!Params.bPreviewCameraShakes)
		{
			CameraShakePreviewUpdater->GetPreviewer().UnRegisterViewModifier(Params.ViewportClient);
		}
	}
}

void SCameraShakePreviewer::OnCameraShakesListSectionChanged(TSharedPtr<FCameraShakeData> Entry, ESelectInfo::Type SelectInfo) const
{
	PlayStopSelectedButton->SetEnabled(Entry != nullptr);

	GEditor->SelectNone(true, true, false);
	if (Entry != nullptr && Entry->SourceComponent.IsValid())
	{
		AActor* SourceActor = Entry->SourceComponent.Get()->GetOwner();
		GEditor->SelectActor(SourceActor, true, true);
	}
}

FText SCameraShakePreviewer::GetActiveViewportName() const
{
	if (ActiveViewportIndex != INDEX_NONE)
	{
		return FText::FromString(LexToString(ActiveViewportIndex + 1));
	}
	return FText::FromString("<None>");
}

FText SCameraShakePreviewer::GetActiveViewportWarnings() const
{
	FText Warnings;
	if (ActiveViewportClient != nullptr)
	{
		FText Delimiter = FText::FromString(", ");
		if (!CameraShakePreviewerModule->HasCameraShakesPreview(ActiveViewportClient))
		{
			// Ya can't see no shakes if yer not previewing no shakes.
			FText Warning = LOCTEXT("ActiveViewportPreviewShakesOffWarning", "Camera shakes previewing is off");
			Warnings = Warnings.IsEmpty() ? Warning : FText::Join(Delimiter, Warnings, Warning);
		}
		if (!ActiveViewportClient->IsRealtime())
		{
			// When real-time mode is off in a viewport, the viewport is ticked only when the user is actively
			// doing something, so the shaking comes and goes in a weird way. Might as well warn the user about it.
			FText Warning = LOCTEXT("ActiveViewportRealtimeOffWarning", "Real-time mode is off");
			Warnings = Warnings.IsEmpty() ? Warning : FText::Join(Delimiter, Warnings, Warning);
		}
	}
	else
	{
		Warnings = LOCTEXT("NoActiveViewportWarning", "No active viewport");
	}
	return Warnings;
}

FReply SCameraShakePreviewer::OnPlayStopAllShakes()
{
	TArray<FActiveCameraShakeInfo> ActiveCameraShakes;
	CameraShakePreviewUpdater->GetActiveCameraShakes(ActiveCameraShakes);
	if (ActiveCameraShakes.Num() > 0)
	{
		// If we have at least 1 shake still playing, stop it.
		CameraShakePreviewUpdater->RemoveAllCameraShakes();

		for (TSharedPtr<FCameraShakeData> CameraShake : CameraShakes)
		{
			CameraShake->ShakeInstance = nullptr;
			CameraShake->bIsPlaying = false;
		}
	}
	else
	{
		// No shakes playing... start them all.
		for (TSharedPtr<FCameraShakeData> CameraShake : CameraShakes)
		{
			if (CameraShake->ShakeClass != nullptr)
			{
				PlayCameraShake(CameraShake);
			}
		}
	}
	
	return FReply::Handled();
}

FReply SCameraShakePreviewer::OnPlayStopSelectedShake()
{
	TArray<TSharedPtr<FCameraShakeData>> SelectedItems = CameraShakesListView->GetSelectedItems();
	for (TSharedPtr<FCameraShakeData> SelectedItem : SelectedItems)
	{
		if (!SelectedItem->bIsPlaying && SelectedItem->ShakeClass != nullptr)
		{
			PlayCameraShake(SelectedItem);
		}
		else if (SelectedItem->bIsPlaying && SelectedItem->ShakeInstance != nullptr)
		{
			CameraShakePreviewUpdater->RemoveCameraShake(SelectedItem->ShakeInstance);
			SelectedItem->ShakeInstance = nullptr;
			SelectedItem->bIsPlaying = false;
		}
	}
	return FReply::Handled();
}

void SCameraShakePreviewer::PlayCameraShake(TSharedPtr<FCameraShakeData> CameraShake)
{
	FAddCameraShakeParams Params;
	Params.SourceComponent = CameraShake->SourceComponent.Get();
	UCameraShakeBase* ShakeInstance = CameraShakePreviewUpdater->AddCameraShake(CameraShake->ShakeClass, Params);
	CameraShake->ShakeInstance = ShakeInstance;
	CameraShake->bIsPlaying = true;
}

void SCameraShakePreviewer::OnLevelViewportClientListChanged()
{
	// When viewports change, our shake previewer already correctly unregisters from any removed viewport.
	// However, we need to automatically register any *new* viewport that fits our requirements.
	if (UWorld* CurrentWorld = WeakCurrentWorld.Get())
	{
		CameraShakePreviewUpdater->GetPreviewer().RegisterViewModifiers(
				[this, CurrentWorld](FLevelEditorViewportClient* LevelVC)
				{
					return LevelVC->GetWorld() == CurrentWorld && 
						CameraShakePreviewerModule->HasCameraShakesPreview(LevelVC);
				},
				// Ignore duplicate registrations.
				true);
	}
}

void SCameraShakePreviewer::OnLevelAdded(ULevel* InLevel, UWorld* InWorld)
{
	Refresh();
}

void SCameraShakePreviewer::OnLevelRemoved(ULevel* InLevel, UWorld* InWorld)
{
	Refresh();
}

void SCameraShakePreviewer::OnLevelActorsAdded(AActor* InActor)
{
	Refresh();
}

void SCameraShakePreviewer::OnLevelActorsRemoved(AActor* InActor)
{
	Refresh();
}

void SCameraShakePreviewer::OnLevelActorListChanged()
{
	Refresh();
}

void SCameraShakePreviewer::OnMapChange(uint32 MapFlags)
{
	Refresh();
}

void SCameraShakePreviewer::OnNewCurrentLevel()
{
	Refresh();
}

void SCameraShakePreviewer::OnMapLoaded(const FString&  Filename, bool bAsTemplate)
{
	Refresh();
}

void SCameraShakePreviewer::PostUndo(bool bSuccess)
{
	Refresh();
}

#undef LOCTEXT_NAMESPACE

