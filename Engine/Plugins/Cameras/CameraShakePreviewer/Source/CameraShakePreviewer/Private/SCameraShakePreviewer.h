// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "EditorUndoClient.h"
#include "TickableEditorObject.h"
#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"
#include "Evaluation/MovieSceneCameraShakePreviewer.h"

class SButton;
class UCameraModifier_CameraShake;
namespace ESelectInfo { enum Type : int; }
struct FActiveCameraShakeInfo;
struct FAddCameraShakeParams;
template <typename ItemType> class SListView;

class AActor;
class ACameraActor;
class FCameraShakePreviewUpdater;
class FCameraShakePreviewerLevelEditorViewportClient;
class FCameraShakePreviewerModule;
class FLevelEditorViewportClient;
class FSceneView;
class ITableRow;
class STableViewBase;
class UCameraAnimInst;
class UCameraShakeBase;
class UCameraShakeSourceComponent;
class ULevel;
class UWorld;
struct FCameraShakeData;
struct FEditorViewportViewModifierParams;
struct FPostProcessSettings;
struct FTogglePreviewCameraShakesParams;

/**
 * Wrapper for the shake previewer that will tick it with the editor.
 */
class FCameraShakePreviewUpdater : public FTickableEditorObject
{
public:
	FCameraShakePreviewUpdater(UWorld* InWorld);
	virtual ~FCameraShakePreviewUpdater();

	// FTickableObject Interface
	virtual ETickableTickType GetTickableTickType() const { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FCameraShakePreviewUpdater, STATGROUP_Tickables); }
	virtual void Tick(float DeltaTime) override;

	UCameraShakeBase* AddCameraShake(TSubclassOf<UCameraShakeBase> ShakeClass, const FAddCameraShakeParams& Params);
	void GetActiveCameraShakes(TArray<FActiveCameraShakeInfo>& ActiveCameraShakes) const;
	void RemoveCameraShake(UCameraShakeBase* ShakeInstance);
	void RemoveAllCameraShakesFromSource(const UCameraShakeSourceComponent* SourceComponent);
	void RemoveAllCameraShakes();

	FCameraShakePreviewer& GetPreviewer() { return Previewer; }

private:
	FCameraShakePreviewer Previewer;
};

/**
 * Camera shake preview panel.
 */
class SCameraShakePreviewer : public SCompoundWidget, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SCameraShakePreviewer) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SCameraShakePreviewer();

	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	// FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

private:
	void Populate();
	void Refresh();
	void UpdateActiveViewportAndWorld();

	void OnTogglePreviewCameraShakes(const FTogglePreviewCameraShakesParams& Params);

	TSharedRef<ITableRow> OnCameraShakesListGenerateRowWidget(TSharedPtr<FCameraShakeData> Entry, const TSharedRef<STableViewBase>& OwnerTable) const;
	void OnCameraShakesListSectionChanged(TSharedPtr<FCameraShakeData> Entry, ESelectInfo::Type SelectInfo) const;
	FText GetActiveViewportName() const;
	FText GetActiveViewportWarnings() const;

	FReply OnPlayStopAllShakes();
	FReply OnPlayStopSelectedShake();
	void PlayCameraShake(TSharedPtr<FCameraShakeData> CameraShake);

	void OnLevelViewportClientListChanged();
	void OnLevelAdded(ULevel* InLevel, UWorld* InWorld);
	void OnLevelRemoved(ULevel* InLevel, UWorld* InWorld);
	void OnLevelActorsAdded(AActor* InActor);
	void OnLevelActorsRemoved(AActor* InActor);
	void OnLevelActorListChanged();
	void OnMapChange(uint32 MapFlags);
	void OnNewCurrentLevel();
	void OnMapLoaded(const FString&  Filename, bool bAsTemplate);

private:
	TArray<TSharedPtr<FCameraShakeData>> CameraShakes;
	TUniquePtr<FCameraShakePreviewUpdater> CameraShakePreviewUpdater;

	TSharedPtr<SListView<TSharedPtr<FCameraShakeData>>> CameraShakesListView;
	TSharedPtr<SButton> PlayStopSelectedButton;

	FCameraShakePreviewerModule* CameraShakePreviewerModule;

	FLevelEditorViewportClient* ActiveViewportClient;
	int32 ActiveViewportIndex;

	TWeakObjectPtr<UWorld> WeakCurrentWorld;

	bool bNeedsRefresh;
};
