// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"

class FExtender;
class FMenuBuilder;
class FUICommandList;
class FLevelEditorViewportClient;
class FSpawnTabArgs;
class SDockTab;
class SWidget;

struct FTogglePreviewCameraShakesParams
{
	FLevelEditorViewportClient* ViewportClient = nullptr;
	bool bPreviewCameraShakes = false;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnTogglePreviewCameraShakes, const FTogglePreviewCameraShakesParams&);

/**
 * Module for managing the camera shake previewer tool.
 */
class FCameraShakePreviewerModule : public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule();
	virtual void ShutdownModule();

	// Toggles whether the given viewport should show camera shakes.
	void ToggleCameraShakesPreview(FLevelEditorViewportClient* ViewportClient);
	// Gets whether the given viewport is supporting camera shakes.
	bool HasCameraShakesPreview(FLevelEditorViewportClient* ViewportClient) const;

public:
	// Callback for when a viewport's ability to show camera shakes is toggled.
	FOnTogglePreviewCameraShakes OnTogglePreviewCameraShakes;
	
private:
	static TSharedRef<SDockTab> CreateCameraShakePreviewerTab(const FSpawnTabArgs& Args);

	void RegisterEditorTab();
	void RegisterViewportOptionMenuExtender();

	void UnregisterEditorTab();
	void UnregisterViewportOptionMenuExtender();

	TSharedRef<FExtender> OnExtendLevelViewportOptionMenu(const TSharedRef<FUICommandList> CommandList);
	void OnLevelViewportClientListChanged();

	void CreateCameraShakeToggleOption(FMenuBuilder& InMenuBuilder);

private:
	FDelegateHandle LevelEditorTabManagerChangedHandle;
	FDelegateHandle ViewportOptionsMenuExtenderHandle;

	struct FViewportInfo
	{
		bool bPreviewCameraShakes = false;
	};
	TMap<FLevelEditorViewportClient*, FViewportInfo> ViewportInfos;
};
